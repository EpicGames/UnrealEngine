{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Remote Control Protocol OSC",
	"Description": "Allows interactions between OSC and RemoteControl API.",
	"Category": "Messaging",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "RemoteControlProtocolOSC",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "RemoteControl",
			"Enabled": true
		},
		{
			"Name": "OSC",
			"Enabled": true
		}
	]
}