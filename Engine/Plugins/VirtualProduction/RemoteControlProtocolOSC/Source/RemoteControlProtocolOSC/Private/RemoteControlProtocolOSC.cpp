// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlProtocolOSC.h"

#include "OSCAddress.h"
#include "RemoteControlLogger.h"

#if WITH_EDITOR
#include "IRCProtocolBindingList.h"
#include "IRemoteControlProtocolWidgetsModule.h"
#endif

#include "OSCManager.h"
#include "OSCMessage.h"

#define LOCTEXT_NAMESPACE "RemoteControlProtocolOSC"

DEFINE_LOG_CATEGORY_STATIC(LogRemoteControlOSC, Log, All);

const FName FRemoteControlProtocolOSC::ProtocolName = TEXT("OSC");

#if WITH_EDITOR

namespace RemoteControlOSCProtocolColumns
{
	static FName Address = TEXT("Address");
}

#endif // WITH_EDITOR

bool FRemoteControlOSCProtocolEntity::IsSame(const FRemoteControlProtocolEntity* InOther)
{
	if(const FRemoteControlOSCProtocolEntity* Other = static_cast<const FRemoteControlOSCProtocolEntity*>(InOther))
	{
		return PathName == Other->PathName;
	}

	return false;
}

#if WITH_EDITOR

void FRemoteControlOSCProtocolEntity::RegisterProperties()
{
	EXPOSE_PROTOCOL_PROPERTY(RemoteControlOSCProtocolColumns::Address, FRemoteControlOSCProtocolEntity, PathName);
}

#endif // WITH_EDITOR

void FRemoteControlProtocolOSC::Bind(FRemoteControlProtocolEntityPtr InRemoteControlProtocolEntityPtr)
{
	if (!ensure(InRemoteControlProtocolEntityPtr.IsValid()))
	{
		return;
	}

	if (FRemoteControlOSCProtocolEntity* OSCProtocolEntity = InRemoteControlProtocolEntityPtr->Cast<FRemoteControlOSCProtocolEntity>())
	{
		TArray<FRemoteControlProtocolEntityWeakPtr>& BindingsEntitiesPtr = Bindings.FindOrAdd(OSCProtocolEntity->PathName);
		BindingsEntitiesPtr.Add(MoveTemp(InRemoteControlProtocolEntityPtr));
	}
	else
	{
		UE_LOGF(LogRemoteControlOSC, Warning, "Invalid Remote Control Protocol Entity.")
	}
}

void FRemoteControlProtocolOSC::Unbind(FRemoteControlProtocolEntityPtr InRemoteControlProtocolEntityPtr)
{
	if (!ensure(InRemoteControlProtocolEntityPtr.IsValid()))
	{
		return;
	}

	if (FRemoteControlOSCProtocolEntity* OSCProtocolEntity = InRemoteControlProtocolEntityPtr->Cast<FRemoteControlOSCProtocolEntity>())
	{
		for (TPair<FName, TArray<FRemoteControlProtocolEntityWeakPtr>>& BindingsPair : Bindings)
		{
			BindingsPair.Value.RemoveAllSwap(CreateProtocolComparator(OSCProtocolEntity->GetPropertyId()));
		}
	}
	else
	{
		UE_LOGF(LogRemoteControlOSC, Warning, "Invalid Remote Control Protocol Entity.")
	}
}

void FRemoteControlProtocolOSC::OSCReceivedMessageEvent(const FOSCMessage& Message, const FString& IPAddress, uint16 Port)
{	
	const FOSCAddress& Address = Message.GetAddress();
	
#if WITH_EDITOR
	ProcessAutoBinding(Address);
#endif
	
	TArray<float> FloatValues;
	UOSCManager::GetAllFloats(Message, FloatValues);

#if WITH_EDITOR
	// Lambda executed only in case of log has been enabled.
	FRemoteControlLogger::Get().Log(ProtocolName, [&IPAddress, &Address, &FloatValues]
	{
		FString FloatValuesString;
		for (int32 FloatValueIndex = 0; FloatValueIndex < FloatValues.Num(); ++FloatValueIndex)
		{
			const FString FloatValueString = FString::SanitizeFloat(FloatValues[FloatValueIndex]);
			if (FloatValueIndex == 0)
			{
				FloatValuesString += FloatValueString;
			}
			else
			{
				FloatValuesString += TEXT(", ") + FloatValueString;
			}
		}
		
		return FText::Format(
			LOCTEXT("OSCEventLog", "IPAddress {0}, FullPath {1}, FloatValues {2}"), FText::FromString(IPAddress),
			FText::FromString(Address.GetFullPath()), FText::FromString(FloatValuesString));
	});
#endif

	const TArray<FRemoteControlProtocolEntityWeakPtr>* ProtocolEntityArrayPtr = Bindings.Find(*Address.GetFullPath());
	if (ProtocolEntityArrayPtr == nullptr)
	{
		return;
	}

	for (const float FloatValue : FloatValues)
	{
		for (const FRemoteControlProtocolEntityWeakPtr& BindingPtr : *ProtocolEntityArrayPtr)
		{
			if (const TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>> ProtocolEntityPtr = BindingPtr.Pin())
			{
				QueueValue(ProtocolEntityPtr, FloatValue);
			}
		}
	}
}

#if WITH_EDITOR
void FRemoteControlProtocolOSC::ProcessAutoBinding(const FOSCAddress& InAddress)
{
	// Bind only in Editor
	if (!GIsEditor)
	{
		return;
	}
	
	IRemoteControlProtocolWidgetsModule& RCWidgetsModule = IRemoteControlProtocolWidgetsModule::Get();
	const TSharedPtr<IRCProtocolBindingList> RCProtocolBindingList = RCWidgetsModule.GetProtocolBindingList();
	if (RCProtocolBindingList.IsValid())
	{
		for (const TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>>& ProtocolEntityPtr : RCProtocolBindingList->GetAwaitingProtocolEntities())
		{
			if (ProtocolEntityPtr.IsValid())
			{
				if ((*ProtocolEntityPtr)->GetBindingStatus() == ERCBindingStatus::Awaiting)
				{
					Unbind(ProtocolEntityPtr);

					if (FRemoteControlOSCProtocolEntity* OSCProtocolEntity = ProtocolEntityPtr->Cast<FRemoteControlOSCProtocolEntity>())
					{
						OSCProtocolEntity->PathName = *InAddress.GetFullPath();						
					}

					Bind(ProtocolEntityPtr);
				}
			}	
		}
	}
}

void FRemoteControlProtocolOSC::RegisterColumns()
{
	FRemoteControlProtocol::RegisterColumns();

	REGISTER_COLUMN(RemoteControlOSCProtocolColumns::Address
		, LOCTEXT("RCPresetAddressColumnHeader", "Address")
		, ProtocolColumnConstants::ColumnSizeMedium);
}

#endif // WITH_EDITOR

void FRemoteControlProtocolOSC::UnbindAll()
{
	Bindings.Empty();
}

#undef LOCTEXT_NAMESPACE
