// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RemoteControlProtocol.h"
#include "RemoteControlProtocolBinding.h"

#include "RemoteControlProtocolOSC.generated.h"

struct FOSCAddress;
struct FOSCMessage;

/**
 * OSC protocol entity for remote control binding
 */
USTRUCT()
struct FRemoteControlOSCProtocolEntity : public FRemoteControlProtocolEntity
{	
	GENERATED_BODY()

public:
	//~ Begin FRemoteControlProtocolEntity interface
	virtual FName GetRangePropertyName() const override { return NAME_FloatProperty; }
	//~ End FRemoteControlProtocolEntity interface

	/** OSC address in the form '/Container1/Container2/Method' */
	UPROPERTY(EditAnywhere, Category = Mapping)
	FName PathName;

	/** OSC range input property template, used for binding. */
	UPROPERTY(Transient, meta = (ClampMin = 0.0, ClampMax = 1.0))
	float RangeInputTemplate = 0.0f;

	/**
	* Checks if this entity has the same values as the Other.
	* Used to check for duplicate inputs.
	*/
	virtual bool IsSame(const FRemoteControlProtocolEntity* InOther) override;

#if WITH_EDITOR

	/** Register(s) all the widgets of this protocol entity. */
	virtual void RegisterProperties() override;

#endif // WITH_EDITOR
};

/**
 * OSC protocol implementation for Remote Control
 */
class FRemoteControlProtocolOSC : public FRemoteControlProtocol
{
public:
	FRemoteControlProtocolOSC()
		: FRemoteControlProtocol(ProtocolName)
	{}
	
	//~ Begin IRemoteControlProtocol interface
	virtual void Bind(FRemoteControlProtocolEntityPtr InRemoteControlProtocolEntityPtr) override;
	virtual void Unbind(FRemoteControlProtocolEntityPtr InRemoteControlProtocolEntityPtr) override;
	virtual void UnbindAll() override;
	virtual UScriptStruct* GetProtocolScriptStruct() const override { return FRemoteControlOSCProtocolEntity::StaticStruct(); }
	//~ End IRemoteControlProtocol interface

	/** Receive OSC server message handler */
	void OSCReceivedMessageEvent(const FOSCMessage& Message, const FString& IPAddress, uint16 Port);

#if WITH_EDITOR
	/**
	 * Process the AutoBinding to the Remote Control Entity
	 * @param InAddress	OSC address structure
	 */
	void ProcessAutoBinding(const FOSCAddress& InAddress);

protected:

	/** Populates protocol specific columns. */
	virtual void RegisterColumns() override;
#endif // WITH_EDITOR

private:
	/** Map of the OSC bindings */
	TMap<FName, TArray<FRemoteControlProtocolEntityWeakPtr>> Bindings;

public:
	/** OSC protocol name */
	static const FName ProtocolName;
};
