// Copyright Epic Games, Inc. All Rights Reserved.


#include "IRemoteControlProtocolModule.h"
#include "RemoteControlProtocolOSC.h"
#include "RemoteControlProtocolOSCSettings.h"

#if WITH_EDITOR
#include "ISettingsModule.h"
#endif

#define LOCTEXT_NAMESPACE "FRemoteControlProtocolOSCModule"

/**
 * OSC remote control module
 */
class FRemoteControlProtocolOSCModule : public IModuleInterface
{
public:
	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface
};

void FRemoteControlProtocolOSCModule::StartupModule()
{
	FModuleManager::Get().LoadModuleChecked(TEXT("OSC"));

#if WITH_EDITOR
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

	// Register OSC Remote Control global settings
	if (SettingsModule != nullptr)
	{
		SettingsModule->RegisterSettings("Project", "Plugins", "Remote Control OSC Protocol",
			LOCTEXT("ProjectSettings_Label", "Remote Control OSC Protocol"),
			LOCTEXT("ProjectSettings_Description", "Configure OSC remote control plugin global settings"),
			GetMutableDefault<URemoteControlProtocolOSCSettings>()
		);
	}
#endif // WITH_EDITOR

	const IRemoteControlProtocolModule& RemoteControlProtocolModule = IRemoteControlProtocolModule::Get();
	if (!RemoteControlProtocolModule.IsRCProtocolsDisable())
	{
		IRemoteControlProtocolModule::Get().AddProtocol(FRemoteControlProtocolOSC::ProtocolName, MakeShared<FRemoteControlProtocolOSC>());

		// Init OSC servers after AddProtocol
		URemoteControlProtocolOSCSettings* OSCSettings = GetMutableDefault<URemoteControlProtocolOSCSettings>();
		OSCSettings->InitOSCServers();
	}
}

void FRemoteControlProtocolOSCModule::ShutdownModule()
{
#if WITH_EDITOR
	// Unregister OSC Remote Control global settings
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
	if (SettingsModule)
	{
		SettingsModule->UnregisterSettings("Project", "Plugins", "Remote Control OSC Protocol");
	}
#endif // WITH_EDITOR
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FRemoteControlProtocolOSCModule, RemoteControlProtocolOSC);
