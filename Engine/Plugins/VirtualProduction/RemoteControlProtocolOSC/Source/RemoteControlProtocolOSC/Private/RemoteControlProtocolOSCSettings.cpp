// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlProtocolOSCSettings.h"

#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "IRemoteControlProtocolModule.h"
#include "OSCManager.h"
#include "OSCMessage.h"
#include "RemoteControlProtocolOSC.h" 
#include "UObject/Package.h"

void FRemoteControlOSCServerSettings::InitOSCServer()
{
	FIPv4Endpoint OSCServerEndpoint;

	if (FIPv4Endpoint::Parse(ServerAddress, OSCServerEndpoint))
	{
		const FString ServerName = FString::Printf(TEXT("FRemoteControlProtocolOSC_Server_%s"), *ServerAddress);
		const bool bMulticastLoopback = OSCServerEndpoint.Address.IsMulticastAddress();
		constexpr bool bStartListening = true;
		OSCServer = TStrongObjectPtr<UOSCServer>(UOSCManager::CreateOSCServer(OSCServerEndpoint.Address.ToString(), OSCServerEndpoint.Port, bMulticastLoopback, bStartListening, ServerName, GetTransientPackage()));
		if (OSCServer.IsValid())
		{
#if WITH_EDITOR
			OSCServer->SetTickInEditor(true);
#endif

			TSharedPtr<FRemoteControlProtocolOSC> ControlProtocolOSC = StaticCastSharedPtr<FRemoteControlProtocolOSC>(IRemoteControlProtocolModule::Get().GetProtocolByName(FRemoteControlProtocolOSC::ProtocolName));
			check(ControlProtocolOSC.IsValid());

			OSCServer->OnOscMessageReceivedNative.AddSP(ControlProtocolOSC.Get(), &FRemoteControlProtocolOSC::OSCReceivedMessageEvent);
		}
	}
}

void URemoteControlProtocolOSCSettings::InitOSCServers()
{
	for (FRemoteControlOSCServerSettings& ServerSettings : ServersSettings)
	{
		ServerSettings.InitOSCServer();
	}
}

#if WITH_EDITOR
void URemoteControlProtocolOSCSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.MemberProperty->GetFName() != PropertyChangedEvent.Property->GetFName() &&
		PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(URemoteControlProtocolOSCSettings, ServersSettings))
	{
		InitOSCServers();
	}
}
#endif
