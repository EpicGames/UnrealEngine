// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "OSCServer.h"
#include "UObject/StrongObjectPtr.h"

#include "RemoteControlProtocolOSCSettings.generated.h"

struct FPropertyChangedEvent;

/**
 * OSC Remote Control server settings
 */
USTRUCT()
struct FRemoteControlOSCServerSettings
{
	GENERATED_BODY();

public:
	/** Initialize OSC servers from ip address and port */
	void InitOSCServer();

public:
	/**
	* OSC server IP address
	* 
	* The format is IP_ADDRESS:PORT_NUMBER.
	*/
	UPROPERTY(EditAnywhere, Category = OSC);
	FString ServerAddress = "127.0.0.1:8001";

	/** Running OSC server instance */
	TStrongObjectPtr<UOSCServer> OSCServer;
};

/**
 * OSC Remote Control settings
 */
UCLASS(Config = Engine, DefaultConfig, meta = (Keywords = "OSC"))
class URemoteControlProtocolOSCSettings : public UObject
{
	GENERATED_BODY()

public:
	/** Initialize all servers from settings at OSCServers map */
	void InitOSCServers();

	//~ Begin UObject interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject interface

public:
	/** OSC server pair of server ip and server port */
	UPROPERTY(Config, EditAnywhere, Category = OSC)
	TArray<FRemoteControlOSCServerSettings> ServersSettings = { FRemoteControlOSCServerSettings() };
};
