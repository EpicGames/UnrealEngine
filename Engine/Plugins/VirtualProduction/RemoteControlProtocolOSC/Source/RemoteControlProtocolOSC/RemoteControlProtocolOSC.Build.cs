// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class RemoteControlProtocolOSC : ModuleRules
{
	public RemoteControlProtocolOSC(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Networking",
				"OSC",
				"RemoteControl",
				"RemoteControlProtocol"
			}
		);

		if (Target.Type == TargetType.Editor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"InputCore",
				    "RemoteControlProtocolWidgets",
					"Settings",
				}
			);
		}
	}
}
